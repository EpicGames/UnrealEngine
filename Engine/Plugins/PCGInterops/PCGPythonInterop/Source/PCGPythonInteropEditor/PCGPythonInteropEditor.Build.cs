// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PCGPythonInteropEditor : ModuleRules
	{
		public PCGPythonInteropEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"Projects",
					"PCG"
				});

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"AssetDefinition",
						"PCGEditor",
						"UnrealEd",
						"PythonScriptPlugin",
						"Slate"
					}
				);
			}
		}
	}
}
