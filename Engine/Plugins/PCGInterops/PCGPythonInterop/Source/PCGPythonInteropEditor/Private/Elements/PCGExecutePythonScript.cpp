// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGExecutePythonScript.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Utils/PCGLogErrors.h"

#include "PCGEditorCommon.h"
#include "PCGPythonInteropEditorModule.h"
#include "PCGGraph.h"

#include "IPythonScriptPlugin.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "PCGExecutePythonScriptElement"

namespace PCGExecutePythonScript::Constants
{
	static const FName ScriptSourceInputPinLabel = "Source";
	static const FText ScriptSourceInputPinTooltip = LOCTEXT("ScriptSourcePinTooltip", "The script may be passed in as an FString attribute input.");
	static constexpr TCHAR ExpectedFileExtension[] = TEXT("py");
}

const FString UPCGExecutePythonScriptSettings::DefaultInlineScript = LOCTEXT("InlineScriptDefault", "print(\"Hello PCG World!\")").ToString();

#if WITH_EDITOR
bool UPCGExecutePythonScriptSettings::CanEditChange(const FProperty* InProperty) const
{
	using namespace PCGExecutePythonScript::Constants;

	if (!InProperty || !Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (ScriptInputMethod == EPCGPythonScriptInputMethod::Input
		&& InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGExecutePythonScriptSettings, ScriptSource))
	{
		if (const UPCGNode* Node = Cast<UPCGNode>(GetOuter()))
		{
			return Node->GetInputPin(ScriptSourceInputPinLabel)->IsConnected() || !IsPinDefaultValueActivated(ScriptSourceInputPinLabel);
		}
	}

	return true;
}

EPCGChangeType UPCGExecutePythonScriptSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName);

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGExecutePythonScriptSettings, ScriptPath))
	{
		ChangeType |= EPCGChangeType::Settings | EPCGChangeType::Structural | EPCGChangeType::Cosmetic;
	}

	return ChangeType;
}

FName UPCGExecutePythonScriptSettings::GetDefaultNodeName() const
{
	return FName(TEXT("ExecutePythonScript"));
}

FText UPCGExecutePythonScriptSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Execute Python Script");
}

FText UPCGExecutePythonScriptSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Execute a Python script from an inline or connected input, or directly from a .py file.");
}

EPCGSettingsType UPCGExecutePythonScriptSettings::GetType() const
{
	return EPCGSettingsType::Generic;
}

FString UPCGExecutePythonScriptSettings::GetAdditionalTitleInformation() const
{
	switch (ScriptInputMethod)
	{
		case EPCGPythonScriptInputMethod::Input:
			return LOCTEXT("InputTitleInformation", "Source Input").ToString();
		case EPCGPythonScriptInputMethod::File:
		{
			return FText::Format(
				LOCTEXT("FileTitleInformation", "Source File: {0}"),
				FText::FromString(FPaths::GetCleanFilename(ScriptPath.FilePath))).ToString();
		}
	}

	return {};
}

TArray<FPCGPinProperties> UPCGExecutePythonScriptSettings::InputPinProperties() const
{
	if (ScriptInputMethod == EPCGPythonScriptInputMethod::Input)
	{
		TArray<FPCGPinProperties> PinProperties;
		FPCGPinProperties& ScriptSourcePin = PinProperties.Emplace_GetRef(PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel, EPCGDataType::Param);
#if WITH_EDITOR
		ScriptSourcePin.Tooltip = PCGExecutePythonScript::Constants::ScriptSourceInputPinTooltip;
#endif // WITH_EDITOR
		return PinProperties;
	}
	else
	{
		return {};
	}
}

TArray<FPCGPinProperties> UPCGExecutePythonScriptSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	FPCGPinProperties& DepPin = PinProperties.Emplace_GetRef(PCGPinConstants::DefaultExecutionDependencyLabel, EPCGDataType::Any);
#if WITH_EDITOR
	DepPin.Tooltip = PCGPinConstants::Tooltips::ExecutionDependencyTooltip;
#endif // WITH_EDITOR
	DepPin.Usage = EPCGPinUsage::DependencyOnly;

	return PinProperties;
}

#endif // WITH_EDITOR

FPCGElementPtr UPCGExecutePythonScriptSettings::CreateElement() const
{
	return MakeShared<FPCGExecutePythonScriptElement>();
}

#if WITH_EDITOR
bool UPCGExecutePythonScriptSettings::DefaultValuesAreEnabled() const
{
	return ScriptInputMethod == EPCGPythonScriptInputMethod::Input;
}

bool UPCGExecutePythonScriptSettings::IsPinDefaultValueEnabled(const FName PinLabel) const
{
	return PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel;
}

bool UPCGExecutePythonScriptSettings::IsPinDefaultValueActivated(const FName PinLabel) const
{
	return PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel && bIsDefaultValueActivated;
}

EPCGMetadataTypes UPCGExecutePythonScriptSettings::GetPinDefaultValueType(const FName PinLabel) const
{
	return EPCGMetadataTypes::String;
}

bool UPCGExecutePythonScriptSettings::IsPinDefaultValueMetadataTypeValid(const FName PinLabel, const EPCGMetadataTypes DataType) const
{
	return DataType == EPCGMetadataTypes::String;
}

bool UPCGExecutePythonScriptSettings::CreateInitialDefaultValueAttribute(const FName PinLabel, UPCGMetadata* OutMetadata) const
{
	check(PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel);
	return OutMetadata && OutMetadata->CreateAttribute<FString>(NAME_None, DefaultInlineScript, /*bAllowsInterpolation=*/false, /*bOverrideParent=*/false);
}

bool UPCGExecutePythonScriptSettings::IsInputPinRequiredByExecution(const UPCGPin* InPin) const
{
	// The pin maintains 'required' status unless the default value is both enabled and activated.
	return InPin && (InPin->IsConnected() || !IsPinDefaultValueEnabled(InPin->Properties.Label) || !IsPinDefaultValueActivated(InPin->Properties.Label));
}

void UPCGExecutePythonScriptSettings::ResetDefaultValues()
{
	ResetDefaultValue(PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel);
}

void UPCGExecutePythonScriptSettings::ResetDefaultValue(const FName PinLabel)
{
	check(PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel);
	SetPinDefaultValue(PinLabel, DefaultInlineScript);
}

void UPCGExecutePythonScriptSettings::SetPinDefaultValue(const FName PinLabel, const FString& DefaultValue, const bool bCreateIfNeeded)
{
	check(PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel);
	Modify();
	InlineScript = DefaultValue;
	OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Node | EPCGChangeType::Edge);
}

void UPCGExecutePythonScriptSettings::SetPinDefaultValueIsActivated(const FName PinLabel, const bool bIsActivated, const bool bDirtySettings)
{
	check(PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel);

	if (bIsDefaultValueActivated != bIsActivated)
	{
		if (bDirtySettings)
		{
			Modify();
		}

		bIsDefaultValueActivated = bIsActivated;

		if (bDirtySettings)
		{
			OnSettingsChangedDelegate.Broadcast(this, EPCGChangeType::Node | EPCGChangeType::Edge);
		}
	}
}

FString UPCGExecutePythonScriptSettings::GetPinDefaultValueAsString(const FName PinLabel) const
{
	check(PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel);
	return InlineScript;
}

FString UPCGExecutePythonScriptSettings::GetPinInitialDefaultValueString(const FName PinLabel) const
{
	check(PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel);
	return DefaultInlineScript;
}

EPCGSettingDefaultValueExtraFlags UPCGExecutePythonScriptSettings::GetDefaultValueExtraFlags(const FName PinLabel) const
{
	check(PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel);
	return EPCGSettingDefaultValueExtraFlags::WideText | EPCGSettingDefaultValueExtraFlags::MultiLineText;
}
#endif // WITH_EDITOR

EPCGMetadataTypes UPCGExecutePythonScriptSettings::GetPinInitialDefaultValueType(const FName PinLabel) const
{
	check(PinLabel == PCGExecutePythonScript::Constants::ScriptSourceInputPinLabel);
	return EPCGMetadataTypes::String;
}

bool FPCGExecutePythonScriptElement::ExecuteInternal(FPCGContext* InContext) const
{
	using namespace PCGExecutePythonScript::Constants;
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGExecutePythonScriptElement::Execute);
	check(InContext);

#ifdef WITH_EDITOR
	const UPCGExecutePythonScriptSettings* Settings = InContext->GetInputSettings<UPCGExecutePythonScriptSettings>();
	check(Settings);

	const IPythonScriptPlugin* PythonModule = IPythonScriptPlugin::Get();
	if (!PythonModule || !PythonModule->IsPythonAvailable())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("PythonNotAvailable", "Python script can not be run. Python Interpreter is not available."), InContext);
		// @todo_pcg: Could try to load or delay execution until its available. Would need max wait time user setting.
		return true;
	}

	FPythonCommandEx PythonSource
	{
		.Flags = EPythonCommandFlags::None,
	};

	TStringBuilder<0> CommandBuilder;

	if (Settings->ScriptInputMethod == EPCGPythonScriptInputMethod::Input)
	{
		if (InContext->InputData.GetInputCountByPin(ScriptSourceInputPinLabel) > 0)
		{
			TArray<FPCGTaggedData> InputData = InContext->InputData.GetInputsByPin(ScriptSourceInputPinLabel);
			for (const FPCGTaggedData& Input : InputData)
			{
				FPCGAttributePropertyInputSelector Selector = Settings->ScriptSource.CopyAndFixLast(Input.Data);
				const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(Input.Data, Selector);
				const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(Input.Data, Selector);
				TArray<FString> Values;
				Values.SetNum(Keys->GetNum());
				Accessor->GetRange(MakeArrayView(Values), 0, *Keys);
				for (const FString& Value : Values)
				{
					CommandBuilder += Value;
					CommandBuilder += LINE_TERMINATOR;
				}
			}
		}
		else if (Settings->IsPinDefaultValueActivated(ScriptSourceInputPinLabel))
		{
			CommandBuilder = Settings->GetPinDefaultValueAsString(ScriptSourceInputPinLabel);
		}
		else // No work to do
		{
			return true;
		}

		PythonSource.ExecutionMode = EPythonCommandExecutionMode::ExecuteStatement;
	}
	else // ScriptInputMethod == EPCGPythonScriptInputMethod::File
	{
		if (Settings->ScriptPath.FilePath.IsEmpty())
		{
			return true;
		}

		if (const FString Extension = FPaths::GetExtension(Settings->ScriptPath.FilePath); !Extension.Equals(ExpectedFileExtension))
		{
			const FString DisplayExtension = Extension.IsEmpty() ? LOCTEXT("NoExtension", "None").ToString() : FString::Printf(TEXT(".%s"), *Extension);
			PCGLog::InputOutput::LogInvalidFileType(DisplayExtension, ExpectedFileExtension, InContext);
			return true;
		}

		if (!IFileManager::Get().FileExists(*Settings->ScriptPath.FilePath))
		{
			PCGLog::InputOutput::LogFileNotFound(Settings->ScriptPath.FilePath, InContext);
			return true;
		}

		CommandBuilder = Settings->ScriptPath.FilePath;
	}

	PythonSource.Command = CommandBuilder;

	if (!IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonSource))
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("PythonScriptHadErrors", "Errors running Python script. See log."));
		for (const FPythonLogOutputEntry& LogOutputEntry : PythonSource.LogOutput)
		{
			switch (LogOutputEntry.Type)
			{
				case EPythonLogOutputType::Info:
					UE_LOGF(LogPCGPython, Log, "Python [Log]: [%ls]", *LogOutputEntry.Output);
					break;
				case EPythonLogOutputType::Warning:
					UE_LOGF(LogPCGPython, Warning, "Python [Warning]: [%ls]", *LogOutputEntry.Output);
					break;
				case EPythonLogOutputType::Error:
					UE_LOGF(LogPCGPython, Error, "Python [Error]: [%ls]", *LogOutputEntry.Output);
					break;
			}
		}
	}

	// Notify the user that a script was run.
	check(InContext->Node);
	check(InContext->Node->GetGraph());

	if (!Settings->bMuteEditorToast)
	{
		FPCGEditorCommon::Helpers::DispatchEditorToast(
			LOCTEXT("PythonScriptExecutionToast", "[PCG] Python"),
			FText::Format(INVTEXT("{0}: {1}"),
				FText::FromString(InContext->Node->GetGraph()->GetName()),
				InContext->Node->GetNodeTitle(EPCGNodeTitleType::ListView)));
	}

#else // WITH_EDITOR
	PCGLog::LogErrorOnGraph(LOCTEXT("NodeIsEditorOnly", "The Execute Python Script node is currently Editor-only and should not be used at runtime."), InContext);
#endif // WITH_EDITOR

	return true;
}

#undef LOCTEXT_NAMESPACE
