// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGPythonDataBridge.h"

#if WITH_EDITOR

void UPCGPythonDataBridge::SetOutputCollection(const FPCGDataCollection& Collection)
{
	OutputCollection = Collection;
	bOutputCollectionSet = true;
}

void UPCGPythonDataBridge::AddToCollection(const UPCGData* InData, FName InPinLabel, TArray<FString> InTags)
{
	FPCGTaggedData& TaggedData = OutputCollection.TaggedData.Emplace_GetRef();
	TaggedData.Data = InData;
	TaggedData.Pin = InPinLabel;
	TaggedData.Tags = TSet<FString>(MoveTemp(InTags));
	bOutputCollectionSet = true;
}

void UPCGPythonDataBridge::Initialize(const FPCGDataCollection& InInputCollection)
{
	InputCollection = InInputCollection;
	OutputCollection.Reset();
	bOutputCollectionSet = false;
}

#endif // WITH_EDITOR
