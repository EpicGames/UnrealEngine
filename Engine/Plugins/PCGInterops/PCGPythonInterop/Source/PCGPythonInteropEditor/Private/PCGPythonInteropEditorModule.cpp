// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGPythonInteropEditorModule.h"
#include "Modules/ModuleManager.h"

class FPCGPythonInteropEditorModule final : public IModuleInterface
{
public:
	//~ IModuleInterface implementation
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}
	//~ End IModuleInterface implementation
};

IMPLEMENT_MODULE(FPCGPythonInteropEditorModule, PCGPythonInteropEditor);

DEFINE_LOG_CATEGORY(LogPCGPython);