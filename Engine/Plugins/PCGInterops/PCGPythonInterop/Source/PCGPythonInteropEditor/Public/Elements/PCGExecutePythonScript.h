// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** @todo_pcg: This node could welcome many improvements in the future, including:
 * - EPythonCommandExecutionMode::EvaluateStatement can provide line-by-line feedback on python.
 * - Optional param inputs/outputs to be get/set from within the Python script, similar to BP and HLSL.
 * - Generalized source editor in PCG for HLSL, Python, and beyond.
 */

#include "PCGSettings.h"
#include "Metadata/PCGDefaultValueInterface.h"

#include "PCGExecutePythonScript.generated.h"

UENUM()
enum class EPCGPythonScriptInputMethod
{
	Input UMETA(Tooltip = "Execute the Python script from an input data or inline."),
	File UMETA(Tooltip = "Execute the Python script from a selected file.")
};

/**
* Execute a Python script, either inline, as input, or from a file.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGExecutePythonScriptSettings : public UPCGSettings, public IPCGSettingsDefaultValueProvider
{
	GENERATED_BODY()

	// The default script to employ until replaced by the user's script.
	static const FString DefaultInlineScript;

public:
#if WITH_EDITOR
	//~Begin UObject interface
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	//~End UObject interface

	//~Begin UPCGSettings interface
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override;
	virtual FString GetAdditionalTitleInformation() const override;
	virtual bool CanCullTaskIfUnwired() const override { return false; }
#endif // WITH_EDITOR

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual bool DefaultValuesAreEnabled() const override;
	virtual bool IsPinDefaultValueEnabled(FName PinLabel) const override;
	virtual bool IsPinDefaultValueActivated(FName PinLabel) const override;
	virtual EPCGMetadataTypes GetPinDefaultValueType(FName PinLabel) const override;
	virtual bool IsPinDefaultValueMetadataTypeValid(FName PinLabel, EPCGMetadataTypes DataType) const override;
	virtual bool CreateInitialDefaultValueAttribute(FName PinLabel, UPCGMetadata* OutMetadata) const override;
	virtual bool IsInputPinRequiredByExecution(const UPCGPin* InPin) const override;

#if WITH_EDITOR
	virtual void ResetDefaultValues() override;
	virtual void ResetDefaultValue(FName PinLabel) override;
	virtual void SetPinDefaultValue(FName PinLabel, const FString& DefaultValue, bool bCreateIfNeeded = false) override;
	virtual void SetPinDefaultValueIsActivated(FName PinLabel, bool bIsActivated, bool bDirtySettings = true) override;
	virtual FString GetPinDefaultValueAsString(FName PinLabel) const override;
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override;
	virtual EPCGSettingDefaultValueExtraFlags GetDefaultValueExtraFlags(FName PinLabel) const override;
#endif // WITH_EDITOR

protected:
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface

public:
	/** The method for receiving the intended Python source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EPCGPythonScriptInputMethod ScriptInputMethod = EPCGPythonScriptInputMethod::Input;

	/** Which attribute to use as a script source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable, EditCondition = "ScriptInputMethod == EPCGPythonScriptInputMethod::Input", EditConditionHides))
	FPCGAttributePropertyInputSelector ScriptSource;

	/** The path to the .py file that will be executed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable, EditCondition = "ScriptInputMethod == EPCGPythonScriptInputMethod::File", EditConditionHides, FilePathFilter = "Python files (*.py)|*.py"))
	FFilePath ScriptPath;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	bool bMuteEditorToast = false;

private:
	// Inline constant script on the input pin to run.
	UPROPERTY()
	FString InlineScript = DefaultInlineScript;

	// Inline constant is enabled for the input pin.
	UPROPERTY()
	bool bIsDefaultValueActivated = true;
};

class FPCGExecutePythonScriptElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	// @todo_pcg: To be confirmed that python should be executed on main thread only.
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	// @todo_pcg: Could be a user option to run once only, etc
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
};

