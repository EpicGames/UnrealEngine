// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGDynamicPins.h"
#include "PCGSettings.h"
#include "Elements/PCGExecutePythonScript.h"
#include "Metadata/PCGDefaultValueInterface.h"

#include "PCGPythonDataProcessor.generated.h"

/**
 * Execute a Python script with direct access to PCG data collections.
 * Dynamic pins accept any PCG data type with multi-connection and multi-data.
 * The user receives an FPCGDataCollection for input and builds one for output.
 */
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGPythonDataProcessorSettings : public UPCGSettings, public IPCGDynamicPinsProvider, public IPCGSettingsDefaultValueProvider
{
	GENERATED_BODY()

	UPCGPythonDataProcessorSettings();

	static const FString DefaultInlineScript;

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override;
	virtual FString GetAdditionalTitleInformation() const override;
	virtual bool CanCullTaskIfUnwired() const override { return false; }
#endif // WITH_EDITOR
	virtual bool HasDynamicPins() const override { return true; }
	virtual bool IsInputPinRequiredByExecution(const UPCGPin* InPin) const override;
	//~End UPCGSettings interface

	//~Begin IPCGDynamicPinsProvider interface
	virtual const FPCGDynamicPinContainer* GetDynamicPinContainer(EPCGPinDirection Direction) const override;
	virtual FPCGDynamicPinContainer* GetMutableDynamicPinContainer(EPCGPinDirection Direction) override;
	virtual TArray<FPCGPinProperties> GetStaticPinProperties(EPCGPinDirection Direction) const override;
#if WITH_EDITOR
	virtual FPCGPinProperties CreateDefaultDynamicPin(EPCGPinDirection Direction) const override;
	virtual bool CanUserRenameDynamicPin(EPCGPinDirection Direction) const override { return true; }
#endif // WITH_EDITOR
	//~End IPCGDynamicPinsProvider interface

	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual bool DefaultValuesAreEnabled() const override;
	virtual bool IsPinDefaultValueEnabled(FName PinLabel) const override;
	virtual bool IsPinDefaultValueActivated(FName PinLabel) const override;
	virtual EPCGMetadataTypes GetPinDefaultValueType(FName PinLabel) const override;
	virtual bool CreateInitialDefaultValueAttribute(FName PinLabel, UPCGMetadata* OutMetadata) const override;
#if WITH_EDITOR
	virtual bool IsPinDefaultValueMetadataTypeValid(FName PinLabel, EPCGMetadataTypes DataType) const override;
	virtual void ResetDefaultValues() override;
	virtual void ResetDefaultValue(FName PinLabel) override;
	virtual void SetPinDefaultValue(FName PinLabel, const FString& DefaultValue, bool bCreateIfNeeded = false) override;
	virtual void SetPinDefaultValueIsActivated(FName PinLabel, bool bIsActivated, bool bDirtySettings = true) override;
	virtual FString GetPinDefaultValueAsString(FName PinLabel) const override;
	virtual FString GetPinInitialDefaultValueString(FName PinLabel) const override;
	virtual EPCGSettingDefaultValueExtraFlags GetDefaultValueExtraFlags(FName PinLabel) const override;
#endif // WITH_EDITOR
	//~End IPCGSettingsDefaultValueProvider interface

	/** The method for receiving the intended Python source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EPCGPythonScriptInputMethod ScriptInputMethod = EPCGPythonScriptInputMethod::Input;

	/** Which attribute to use as a script source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable, EditCondition = "ScriptInputMethod == EPCGPythonScriptInputMethod::Input", EditConditionHides))
	FPCGAttributePropertyInputSelector ScriptSource;

	/** The path to the .py file that will be executed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable, EditCondition = "ScriptInputMethod == EPCGPythonScriptInputMethod::File", EditConditionHides, FilePathFilter = "Python files (*.py)|*.py"))
	FFilePath ScriptPath;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable))
	bool bMuteEditorToast = false;

#if WITH_EDITOR
	UFUNCTION()
	static bool SupportsComposition() { return true; }
#endif // WITH_EDITOR

	UPROPERTY()
	FPCGDynamicPinContainer InputDynamicPins;

	UPROPERTY()
	FPCGDynamicPinContainer OutputDynamicPins;

protected:
#if WITH_EDITOR
	//~Begin UObject interface
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
	//~End UObject interface
#endif // WITH_EDITOR

	//~Begin UPCGSettings interface
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

	//~Begin IPCGSettingsDefaultValueProvider interface
	virtual EPCGMetadataTypes GetPinInitialDefaultValueType(FName PinLabel) const override;
	//~End IPCGSettingsDefaultValueProvider interface

private:
	UPROPERTY()
	FString InlineScript = DefaultInlineScript;

	UPROPERTY()
	bool bIsDefaultValueActivated = true;
};

class FPCGPythonDataProcessorElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
};
