// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "PCGData.h"

#include "PCGPythonDataBridge.generated.h"

/**
 * Lightweight UObject bridge for passing FPCGDataCollection (a USTRUCT) into Python scope.
 * Created per-execution with a unique name so Python can locate it via unreal.find_object().
 * Used by the Python Data Processor node to give users direct access to input/output collections.
 *
 * @todo_pcg: This bridge exists because PythonScriptPlugin's PyConversion headers are Private,
 * so we can't use PyConversion::NativizeProperty to transfer USTRUCTs directly between Python and C++.
 * If those headers become public, this class can be removed in favor of direct struct marshalling.
 */
UCLASS(BlueprintType)
class UPCGPythonDataBridge : public UObject
{
	GENERATED_BODY()

public:
	/** Returns the input data collection for the current execution. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Python")
	FPCGDataCollection GetInputCollection() const { return InputCollection; }

	/** Stores the output data collection built by the user's Python script. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Python")
	void SetOutputCollection(const FPCGDataCollection& Collection);

	/** Adds data to the output collection, tagged to the given pin label. Mirrors UPCGDataFunctionLibrary::AddToCollection. */
	UFUNCTION(BlueprintCallable, Category = "PCG|Python")
	void AddToCollection(const UPCGData* InData, FName InPinLabel, TArray<FString> InTags);

	/** Returns true if SetOutputCollection was called during this execution. */
	bool HasOutputCollection() const { return bOutputCollectionSet; }

	/** Returns the output collection set by the user's script. Only valid if HasOutputCollection() is true. */
	const FPCGDataCollection& GetOutputCollection() const { return OutputCollection; }

	/** Populates the bridge with input data before script execution. */
	void Initialize(const FPCGDataCollection& InInputCollection);

private:
	UPROPERTY()
	FPCGDataCollection InputCollection;

	UPROPERTY()
	FPCGDataCollection OutputCollection;

	bool bOutputCollectionSet = false;
};

#endif // WITH_EDITOR
