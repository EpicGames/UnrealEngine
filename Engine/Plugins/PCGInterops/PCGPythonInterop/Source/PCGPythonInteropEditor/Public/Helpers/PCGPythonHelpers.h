// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"

namespace PCG::Python::Helpers
{
	/** Extracts the concise error line from a full Python traceback. */
	inline FString ExtractErrorSummary(const FString& Traceback)
	{
		TArray<FString> Lines;
		Traceback.ParseIntoArrayLines(Lines);

		for (int32 Index = Lines.Num() - 1; Index >= 0; --Index)
		{
			const FString& Line = Lines[Index];

			if (Line.IsEmpty() || Line.TrimStart() != Line)
			{
				continue;
			}

			// Skip the "Traceback (most recent call last):" header line — we want the actual exception.
			if (Line.StartsWith(TEXT("Traceback ")))
			{
				continue;
			}

			return Line;
		}

		return Traceback;
	}
}

#endif // WITH_EDITOR
